{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Sound Cue Templates",
	"Description": "Collection of SoundCue Templates, which provide rapid design of common audio design workflows.",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "SoundCueTemplates",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name" : "SoundCueTemplatesEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostDefault"
		}
		
	]
}