// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundCueContainer.h"

#include "Sound/SoundNodeConcatenator.h"
#include "Sound/SoundNodeMixer.h"
#include "Sound/SoundNodeModulator.h"
#include "Sound/SoundNodeQualityLevel.h"
#include "Sound/SoundNodeRandom.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "SoundCueTemplateSettings.h"
#include "SoundCueTemplatesModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundCueContainer)

#if WITH_EDITORONLY_DATA
#include "DetailLayoutBuilder.h"
#include "PropertyEditorModule.h"
#endif // WITH_EDITORONLY_DATA

#define LOCTEXT_NAMESPACE "SoundCueContainer"

USoundCueContainer::USoundCueContainer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bLooping = false;

	ContainerType = ESoundContainerType::Randomize;

	PitchModulation.X = 0.95;
	PitchModulation.Y = 1.05;

	// Roughly -3dB to 0 dB
	VolumeModulation.X = 0.70f;
	VolumeModulation.Y = 1.00f;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITORONLY_DATA
void USoundCueContainer::OnRebuildGraph(USoundCue& SoundCue) const
{
	const USoundCueTemplateSettings* TemplateSettings = GetDefault<USoundCueTemplateSettings>();
	if (!TemplateSettings)
	{
		return;
	}

	const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
	if (!AudioSettings)
	{
		return;
	}

	if (TemplateSettings->GetQualityLevelSettingsNum() != AudioSettings->GetQualityLevelSettingsNum())
	{
		UE_LOGF(SoundCueTemplates, Warning, "Rebuild SoundCueTemplate 'SoundCueContainer' failed: "
			"Quality settings mismatch between Audio Settings & SoundCueTemplate settings.")
		return;
	}

	if (!Variations.Num())
	{
		return;
	}

	USoundNodeModulator& ModulatorNode = ConstructSoundNodeRoot<USoundNodeModulator>(SoundCue);
	ModulatorNode.PitchMin = PitchModulation.X;
	ModulatorNode.PitchMax = PitchModulation.Y;
	ModulatorNode.VolumeMin = VolumeModulation.X;
	ModulatorNode.VolumeMax = VolumeModulation.Y;

	int32 Column = 1;
	int32 Row = 0;
	const int32 InputPinIndex = 0;
	USoundNodeQualityLevel& QualityNode = USoundCueTemplate::ConstructSoundNodeChild<USoundNodeQualityLevel>(SoundCue, &ModulatorNode, Column, Row, InputPinIndex);
	Column++;

	// Optimization to just link to single wave player and omit randomizer if just one variation provided.
	USoundNodeWavePlayer* WavePlayerNode = nullptr;

	int32 NumQualitySettings = AudioSettings->GetQualityLevelSettingsNum();

	// Get max variations
	int32 TotalWavePlayers = 0;
	for (int32 i = 0; i < NumQualitySettings; ++i)
	{
		const FSoundCueTemplateQualitySettings& QualitySettings = TemplateSettings->GetQualityLevelSettings(i);
		const int32 MaxVariations = GetMaxVariations(QualitySettings);
		TotalWavePlayers = FMath::Max(TotalWavePlayers, MaxVariations);
	}

	// Prune variations to only include valid (non-null) references
	TArray<USoundWave*> VariationArray = Variations.Array();
	for (int32 i = Variations.Num() -1 ; i >= 0; --i)
	{
		if (!VariationArray[i])
		{
			VariationArray.RemoveAtSwap(i, EAllowShrinking::No);
		}
	}

	// Trim total to the size of possible variations & create wave player nodes
	TotalWavePlayers = FMath::Min(TotalWavePlayers, VariationArray.Num());
	TArray<USoundNodeWavePlayer*> WavePlayers;
	const int32 WavePlayerColumn = 4;
	for (int32 i = 0; i < TotalWavePlayers; ++i)
	{
		const int32 WavePlayerRow = i;
		USoundNodeWavePlayer* WavePlayer = &USoundCueTemplate::ConstructSoundNodeChild<USoundNodeWavePlayer>(SoundCue, nullptr, WavePlayerColumn, WavePlayerRow);
		WavePlayers.Add(WavePlayer);
		WavePlayer->SetSoundWave(VariationArray[i]);
		WavePlayer->bLooping = bLooping;
	}

	// Map of max channels to container to avoid duplicating container nodes
	TMap<int32, USoundNode*> ContainerMap;

	int32 ContainerRow = 0;
	for (int32 i = 0; i < NumQualitySettings; ++i)
	{
		const int32 PinIndex = i;
		const FAudioQualitySettings& AudioQualitySettings = AudioSettings->GetQualityLevelSettings(i);
		const FSoundCueTemplateQualitySettings& QualitySettings = TemplateSettings->GetQualityLevelSettings(i);
		const int32 MaxVariations = GetMaxVariations(QualitySettings);
		if (WavePlayers.Num() == 1)
		{
			USoundCueTemplate::AddSoundNodeChild(QualityNode, *WavePlayers[0], PinIndex, &QualitySettings.DisplayName);
		}
		else
		{
			USoundNode* ContainerNode = ContainerMap.FindRef(MaxVariations);
			if (ContainerNode)
			{
				USoundCueTemplate::AddSoundNodeChild(QualityNode, *ContainerNode, PinIndex, &QualitySettings.DisplayName);
			}
			else
			{
				switch (ContainerType)
				{
				case ESoundContainerType::Concatenate:
				{
					ContainerNode = &USoundCueTemplate::ConstructSoundNodeChild<USoundNodeConcatenator>(
						SoundCue,
						&QualityNode,
						Column,
						ContainerRow,
						i,
						&QualitySettings.DisplayName);
				}
				break;

				case ESoundContainerType::Mix:
				{
					ContainerNode = &USoundCueTemplate::ConstructSoundNodeChild<USoundNodeMixer>(
						SoundCue,
						&QualityNode,
						Column,
						ContainerRow,
						i,
						&QualitySettings.DisplayName);
				}
				break;

				case ESoundContainerType::Randomize:
				default:
				{
					ContainerNode = &USoundCueTemplate::ConstructSoundNodeChild<USoundNodeRandom>(
						SoundCue,
						&QualityNode,
						Column,
						ContainerRow,
						i,
						&QualitySettings.DisplayName);
				}
				break;
				}
				check(ContainerNode);

				// Add wave players and connect to random node
				for (int32 j = 0; j < MaxVariations && j < WavePlayers.Num(); ++j)
				{
					USoundCueTemplate::AddSoundNodeChild(*ContainerNode, *WavePlayers[j], j);
				}

				ContainerMap.Add(MaxVariations, ContainerNode);
				ContainerRow++;
			}
		}
	}
}

int32 USoundCueContainer::GetMaxVariations(const FSoundCueTemplateQualitySettings& QualitySettings) const
{
	switch (ContainerType)
	{
		case ESoundContainerType::Concatenate:
		{
			return QualitySettings.MaxConcatenatedVariations;
		}

		case ESoundContainerType::Mix:
		{
			return QualitySettings.MaxMixVariations;
		}

		case ESoundContainerType::Randomize:
		default:
		{
			return QualitySettings.MaxRandomizedVariations;
		}
	}
}

TSet<FName>& USoundCueContainer::GetCategoryAllowList()
{
	static TSet<FName> Categories;
	if (Categories.Num() == 0)
	{
		Categories.Add(FName(TEXT("Modulation")));
		Categories.Add(FName(TEXT("Variation")));
	}

	return Categories;
}

TSharedRef<IDetailCustomization> FSoundCueContainerDetailCustomization::MakeInstance()
{
	return MakeShareable(new FSoundCueContainerDetailCustomization);
}

void FSoundCueContainerDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	TArray<FName> CategoryNames;
	DetailLayout.GetCategoryNames(CategoryNames);

	const TSet<FName>& AllowList = USoundCueContainer::GetCategoryAllowList();
	for (FName CategoryName : CategoryNames)
	{
		if (!AllowList.Contains(CategoryName))
		{
			DetailLayout.HideCategory(CategoryName);
		}
	}
}

void FSoundCueContainerDetailCustomization::Register(FPropertyEditorModule& PropertyModule)
{
	PropertyModule.RegisterCustomClassLayout("SoundCueContainer", FOnGetDetailCustomizationInstance::CreateStatic(&FSoundCueContainerDetailCustomization::MakeInstance));
}

void FSoundCueContainerDetailCustomization::Unregister(FPropertyEditorModule& PropertyModule)
{
	PropertyModule.UnregisterCustomClassLayout("SoundCueContainer");
}
#endif // WITH_EDITORONLY_DATA
#undef LOCTEXT_NAMESPACE

