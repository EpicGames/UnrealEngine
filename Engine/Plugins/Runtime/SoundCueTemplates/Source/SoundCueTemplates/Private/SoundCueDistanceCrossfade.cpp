// Copyright Epic Games, Inc. All Rights Reserved.


#include "SoundCueDistanceCrossfade.h"


#include "Sound/SoundNodeModulator.h"
#include "Sound/SoundNodeWavePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundCueDistanceCrossfade)

#if WITH_EDITOR
#include "DetailLayoutBuilder.h"
#include "PropertyEditorModule.h"
#endif // WITH_EDITOR


USoundCueDistanceCrossfade::USoundCueDistanceCrossfade(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
// Add editor data initialization here
	bLooping = true;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
#define LOCTEXT_NAMESPACE "SoundCueDistanceCrossfade"

void USoundCueDistanceCrossfade::OnRebuildGraph(USoundCue& SoundCue) const
{
	const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
	if (!AudioSettings)
	{
		return;
	}

	USoundNodeModulator& ModulatorNode = USoundCueTemplate::ConstructSoundNodeRoot<USoundNodeModulator>(SoundCue);
	ModulatorNode.PitchMin  = 0.95f;
	ModulatorNode.PitchMax  = 1.05f;

	// Roughly -3dB to 0 dB
	ModulatorNode.VolumeMin = 0.75f;
	ModulatorNode.VolumeMax = 1.00f;

	int32 Column = 1; // 1 as root is '0', which was added above.
	int32 LeafIndex = 0;

	// Building the next two generations in the following nested loops,
	// so begin tracking both totals overall here.
	int32 WavePlayerRow = 0;

	const int32 Row = 0;
	const int32 InputPinIndex = 0;
	USoundNodeDistanceCrossFade& CrossfadeNode = USoundCueTemplate::ConstructSoundNodeChild<USoundNodeDistanceCrossFade>(
		SoundCue,
		&ModulatorNode,
		Column,
		Row,
		InputPinIndex);

	// Create wave players, update crossfade input, and connect to CrossfadeNode
	int i = 0;
	for (const FSoundCueCrossfadeInfo& CrossfadeInfo : Variations)
	{
		if (CrossfadeInfo.Sound)
		{
			const int32 NewIndex = CrossfadeNode.ChildNodes.Num();

			USoundNodeWavePlayer* WavePlayerNode = &USoundCueTemplate::ConstructSoundNodeChild<USoundNodeWavePlayer>(
				SoundCue,
				&CrossfadeNode,
				Column + 1,
				WavePlayerRow,
				i++);

			WavePlayerNode->SetSoundWave(CrossfadeInfo.Sound);
			WavePlayerNode->bLooping = bLooping;
			CrossfadeNode.CrossFadeInput[NewIndex] = CrossfadeInfo.DistanceInfo;
			WavePlayerRow++;
		}
	}
}

TSet<FName>& USoundCueDistanceCrossfade::GetCategoryAllowList()
{
	static TSet<FName> Categories;
	if (Categories.Num() == 0)
	{
		// Add exposed categories here that you'd like to show
		// up in the editor (should match category names properties
		// are assigned to in the header, otherwise the properties
		// will be hidden by default).  If you would like to expose
		// default property categories inherited by USoundCue,
		// add them here as well.
		Categories.Add(FName(TEXT("Wave Parameters")));
	}

	return Categories;
}

TSharedRef<IDetailCustomization> FSoundCueDistanceCrossfadeDetailCustomization::MakeInstance()
{
	return MakeShareable(new FSoundCueDistanceCrossfadeDetailCustomization);
}

void FSoundCueDistanceCrossfadeDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	TArray<FName> CategoryNames;
	DetailLayout.GetCategoryNames(CategoryNames);

	const TSet<FName>& AllowList = USoundCueDistanceCrossfade::GetCategoryAllowList();
	for (FName CategoryName : CategoryNames)
	{
		if (!AllowList.Contains(CategoryName))
		{
			DetailLayout.HideCategory(CategoryName);
		}
	}
}

void FSoundCueDistanceCrossfadeDetailCustomization::Register(FPropertyEditorModule& PropertyModule)
{
	PropertyModule.RegisterCustomClassLayout("SoundCueDistanceCrossfade", FOnGetDetailCustomizationInstance::CreateStatic(&FSoundCueDistanceCrossfadeDetailCustomization::MakeInstance));
}
#undef LOCTEXT_NAMESPACE
#endif // WITH_EDITOR

