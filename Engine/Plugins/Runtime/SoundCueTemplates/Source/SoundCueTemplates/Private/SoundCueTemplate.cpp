// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundCueTemplate.h"
#include "ObjectEditorUtils.h"
#include "Sound/SoundWave.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundCueTemplate)

namespace SoundCueTemplateConsoleVariables
{
	float NodeWidthOffset = 500.f;
	FAutoConsoleVariableRef CVarNodeWidthOffset(
		TEXT("au.SoundCueTemplate.NodeWidthOffset"),
		NodeWidthOffset,
		TEXT("Sets the width between nodes in SoundCues generated from SoundCueTemplates"),
		ECVF_Default);

	float InitialWidthOffset = 300.f;
	FAutoConsoleVariableRef CVarInitialWidthOffset(
		TEXT("au.SoundCueTemplate.InitialWidthOffset"),
		InitialWidthOffset,
		TEXT("Sets the initial width offset of first node in a SoundCue generated from a SoundCueTemplate"),
		ECVF_Default);

	float NodeHeightOffset = 200.f;
	FAutoConsoleVariableRef CVarNodeHeightOffset(
		TEXT("au.SoundCueTemplate.NodeHeightOffset"),
		NodeHeightOffset,
		TEXT("Sets the height between nodes in SoundCues generated from SoundCueTemplates"),
		ECVF_Default);

	float InitialHeightOffset = 0.f;
	FAutoConsoleVariableRef CVarInitialHeightOffset(
		TEXT("au.SoundCueTemplate.InitialHeightOffset"),
		InitialHeightOffset,
		TEXT("Sets the initial height offset of first node in a SoundCue generated from a SoundCueTemplate"),
		ECVF_Default);
} // namespace SoundCueTemplateConsoleVariables

USoundCueTemplate::USoundCueTemplate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR
void USoundCueTemplate::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		RebuildGraph(*this);
	}
}

void USoundCueTemplate::RebuildGraph(USoundCue& SoundCue) const
{
	SoundCue.ResetGraph();

	OnRebuildGraph(SoundCue);

	SoundCue.LinkGraphNodesFromSoundNodes();
	SoundCue.PostEditChange();
	SoundCue.MarkPackageDirty();
}

void USoundCueTemplate::AddSoundWavesToTemplate(const TArray<UObject*>& SelectedObjects)
{
	TArray<TWeakObjectPtr<USoundWave>> Waves(FObjectEditorUtils::GetTypedWeakObjectPtrs<USoundWave>(SelectedObjects));
	AddSoundWaves(Waves);
}

float USoundCueTemplate::GetNodeWidthOffset()
{
	return SoundCueTemplateConsoleVariables::NodeWidthOffset;
}

float USoundCueTemplate::GetInitialWidthOffset()
{
	return SoundCueTemplateConsoleVariables::InitialWidthOffset;
}

float USoundCueTemplate::GetNodeHeightOffset()
{
	return SoundCueTemplateConsoleVariables::NodeHeightOffset;
}

float USoundCueTemplate::GetInitialHeightOffset()
{
	return SoundCueTemplateConsoleVariables::InitialHeightOffset;
}

#endif // WITH_EDITOR

