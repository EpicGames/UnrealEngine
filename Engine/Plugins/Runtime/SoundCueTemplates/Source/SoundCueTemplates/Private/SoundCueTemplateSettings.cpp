// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundCueTemplateSettings.h"

#include "Sound/AudioSettings.h"
#include "UObject/UObjectIterator.h"
#include "SoundCueContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundCueTemplateSettings)

#if WITH_EDITOR

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "SoundCueTemplateSettings"

#if WITH_EDITOR
namespace
{
	void PostNotification(const FText& NotificationText)
	{
		const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
		check(AudioSettings);

		FNotificationInfo Info(NotificationText);
		Info.bFireAndForget = true;
		Info.ExpireDuration = 2.0f;
		Info.bUseThrobber = true;
		FSlateNotificationManager::Get().AddNotification(Info);
	}

	void RebuildNodeGraphs(const ESoundContainerType* ContainerType)
	{
		for (TObjectIterator<USoundCueContainer> It; It; ++It)
		{
			if (USoundCueContainer* Container = *It)
			{
				if (!ContainerType || *ContainerType == Container->ContainerType)
				{
					Container->RebuildGraph(*Container);
				}
			}
		}
	}
} // namespace <>


void FSoundCueTemplateQualitySettingsNotifier::PostQualitySettingsUpdated() const
{
	const FText Text = LOCTEXT("NewSoundCueTemplateQualityLevel", "Main Audio Settings Quality Settings level has been added. SoundCueTemplate Quality Settings may need updates.");
	PostNotification(Text);
}

void FSoundCueTemplateQualitySettingsNotifier::OnAudioSettingsChanged() const
{
	USoundCueTemplateSettings* TemplateSettings = GetMutableDefault<USoundCueTemplateSettings>();
	check(TemplateSettings);
	if (TemplateSettings->RebuildQualityLevels())
	{
		PostQualitySettingsUpdated();
	}
}
#endif // WITH_EDITOR

FSoundCueTemplateQualitySettings::FSoundCueTemplateQualitySettings()
	: MaxConcatenatedVariations(8)
	, MaxRandomizedVariations(8)
	, MaxMixVariations(8)
{
}

USoundCueTemplateSettings::USoundCueTemplateSettings(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

#if WITH_EDITOR
int32 USoundCueTemplateSettings::GetQualityLevelSettingsNum() const
{
	return QualityLevels.Num();
}

const FSoundCueTemplateQualitySettings& USoundCueTemplateSettings::GetQualityLevelSettings(int32 Index) const
{
	return QualityLevels[Index];
}

void USoundCueTemplateSettings::PostInitProperties()
{
	Super::PostInitProperties();

	UAudioSettings* AudioSettings = GetMutableDefault<UAudioSettings>();
	check(AudioSettings);

	AudioSettings->OnAudioSettingsChanged().AddRaw(&SettingsNotifier, &FSoundCueTemplateQualitySettingsNotifier::OnAudioSettingsChanged);
}

void USoundCueTemplateSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (!PropertyChangedEvent.Property)
	{
		return;
	}

	if (!RebuildQualityLevels())
	{
		return;
	}

	MarkPackageDirty();

	bool bUpdateTemplates = false;
	bool bFilterByContainerType = false;
	ESoundContainerType ContainerType = ESoundContainerType::Randomize;

	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FSoundCueTemplateQualitySettings, MaxConcatenatedVariations))
	{
		bUpdateTemplates = true;
		bFilterByContainerType = true;
		ContainerType = ESoundContainerType::Concatenate;
	}
	else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FSoundCueTemplateQualitySettings, MaxRandomizedVariations))
	{
		bUpdateTemplates = true;
		bFilterByContainerType = true;
		ContainerType = ESoundContainerType::Randomize;
	}
	else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FSoundCueTemplateQualitySettings, MaxMixVariations))
	{
		bUpdateTemplates = true;
		bFilterByContainerType = true;
		ContainerType = ESoundContainerType::Mix;
	}
	else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FSoundCueTemplateQualitySettings, DisplayName))
	{
		bUpdateTemplates = true;
	}

	if (bUpdateTemplates)
	{
		RebuildNodeGraphs(bFilterByContainerType ? &ContainerType : nullptr);
	}
}

bool USoundCueTemplateSettings::RebuildQualityLevels()
{
	const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
	check(AudioSettings);

	bool bHasChanged = false;

	while (QualityLevels.Num() > AudioSettings->GetQualityLevelSettingsNum())
	{
		bHasChanged = true;
		QualityLevels.RemoveAt(QualityLevels.Num() - 1);
	}

	while (QualityLevels.Num() < AudioSettings->GetQualityLevelSettingsNum())
	{
		bHasChanged = true;
		QualityLevels.Add(FSoundCueTemplateQualitySettings());
	}

	for (int i = 0; i < AudioSettings->GetQualityLevelSettingsNum(); ++i)
	{
		FText& QualityName = QualityLevels[i].DisplayName;
		const FText& AudioSettingsQualityName = AudioSettings->GetQualityLevelSettings(i).DisplayName;
		QualityName = AudioSettingsQualityName;
	}

	return bHasChanged;
}
#endif // WITH_EDITOR
#undef LOCTEXT_NAMESPACE

