// Copyright Epic Games, Inc. All Rights Reserved.
#include "SoundCueTemplatesModule.h"

#include "Modules/ModuleManager.h"
#include "SoundCueContainer.h"
#include "SoundCueDistanceCrossfade.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#include "PropertyEditorModule.h"
#endif // WITH_EDITOR


DEFINE_LOG_CATEGORY(SoundCueTemplates);

#define LOCTEXT_NAMESPACE "FSoundCueTemplatesModule"
void FSoundCueTemplatesModule::StartupModule()
{
#if WITH_EDITOR
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FSoundCueContainerDetailCustomization::Register(PropertyModule);
	FSoundCueDistanceCrossfadeDetailCustomization::Register(PropertyModule);
#endif //WITH_EDITOR
}

void FSoundCueTemplatesModule::ShutdownModule()
{
#if WITH_EDITOR
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "SoundCueTemplates");
	}
#endif //WITH_EDITOR
}
#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FSoundCueTemplatesModule, SoundCueTemplates)
