// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SoundCueTemplate.h"

#if WITH_EDITOR
#include "IDetailCustomization.h"
#endif // WITH_EDITOR

#include "SoundCueContainer.generated.h"

class FPropertyEditorModule;
struct FSoundCueTemplateQualitySettings;

// ========================================================================
// USoundCueContainer
// Sound Cue template class which implements USoundCueTemplate.
//
// Simple example showing how to expose or hide template
// parameters in the editor such as the looping and soundwave
// fields of a USoundNodeWavePlayer.
//
// In order for proper data hiding to occur for inherited properties,
// Customization Detail's 'Register' must be called in during initialization
// (eg. in module's StartupModule()) like so:
// #include "SoundCueContainer.h"
// ...
// FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
// FSoundCueContainerDetailCustomization::Register(PropertyModule);
// ========================================================================


UENUM()
enum class ESoundContainerType : uint8
{
	Concatenate,
	Randomize,
	Mix
};

UCLASS(MinimalAPI, hidecategories = object, BlueprintType)
class USoundCueContainer : public USoundCueTemplate
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
public:
	UPROPERTY(EditAnywhere, Category = "Variation")
	ESoundContainerType ContainerType;

	UPROPERTY(EditAnywhere, Category = "Variation")
	bool bLooping;

	UPROPERTY(EditAnywhere, Category = "Variation")
	TSet<TObjectPtr<USoundWave>> Variations;

	UPROPERTY(EditAnywhere, Category = "Modulation")
	FVector2D PitchModulation;

	UPROPERTY(EditAnywhere, Category = "Modulation")
	FVector2D VolumeModulation;
#endif // WITH_EDITORONLYDATA

#if WITH_EDITOR
	/** List of categories to allow showing on the inherited SoundCue. */
	static SOUNDCUETEMPLATES_API TSet<FName>& GetCategoryAllowList();
	SOUNDCUETEMPLATES_API virtual void OnRebuildGraph(USoundCue& SoundCue) const override;

private:
	int32 GetMaxVariations(const FSoundCueTemplateQualitySettings& QualitySettings) const;
#endif // WITH_EDITOR
};

#if WITH_EDITOR
class FSoundCueContainerDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	static void Register(FPropertyEditorModule& PropertyModule);
	static void Unregister(FPropertyEditorModule& PropertyModule);
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};
#endif // WITH_EDITOR
