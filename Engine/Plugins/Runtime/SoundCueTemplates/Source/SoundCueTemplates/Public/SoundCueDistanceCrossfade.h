// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SoundCueTemplate.h"
#include "Sound/SoundNodeDistanceCrossFade.h"

#if WITH_EDITOR
#include "IDetailCustomization.h"
#endif // WITH_EDITOR

#include "SoundCueDistanceCrossfade.generated.h"

class FPropertyEditorModule;

// ========================================================================
// USoundCueDistanceCrossfade
// Sound Cue template class which implements USoundCueTemplate.
//
// Simple near/distant mix design for provided, spatialized sounds.
// ========================================================================


USTRUCT(BlueprintType)
struct FSoundCueCrossfadeInfo
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Crossfade Info")
	FDistanceDatum DistanceInfo;

	UPROPERTY(EditAnywhere, Category = "Crossfade Info")
	TObjectPtr<USoundWave> Sound = nullptr;
};

UCLASS(MinimalAPI, hidecategories = object, BlueprintType)
class USoundCueDistanceCrossfade : public USoundCueTemplate
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
public:
	UPROPERTY(EditAnywhere, Category = "Wave Parameters")
	bool bLooping;

	UPROPERTY(EditAnywhere, Category = "Wave Parameters")
	TArray<FSoundCueCrossfadeInfo> Variations;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	/** List of categories to allow showing on the inherited SoundCue. */
	static SOUNDCUETEMPLATES_API TSet<FName>& GetCategoryAllowList();

	/** Overload with logic to rebuild the node graph when
	  * user-facing properties are changed
	  */
	SOUNDCUETEMPLATES_API virtual void OnRebuildGraph(USoundCue& SoundCue) const override;
#endif // WITH_EDITOR
};

#if WITH_EDITOR
class FSoundCueDistanceCrossfadeDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	static void Register(FPropertyEditorModule& PropertyModule);
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};
#endif // WITH_EDITOR
