// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Sound/SoundCue.h"

#if WITH_EDITOR
#include "EdGraph/EdGraphPin.h"
#endif // WITH_EDITOR

#include "SoundCueTemplate.generated.h"


// Base Sound Cue Template class, which builds the sound node graph procedurally and hides more complex Sound Cue functionality
// to streamline implementation defined in child classes.
UCLASS(MinimalAPI, abstract)
class USoundCueTemplate : public USoundCue
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	SOUNDCUETEMPLATES_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

public:
	/** Rebuilds the graph when user-facing properties are changed */
	SOUNDCUETEMPLATES_API void RebuildGraph(USoundCue& SoundCue) const;

	/** Override to initialize a template with array of sound waves from SoundCueFactory */
	virtual void AddSoundWaves(TArray<TWeakObjectPtr<USoundWave>>& Waves) {}

	/** Override to load template default settings from SoundCueTemplateFactory */
	virtual void LoadTemplateDefaultSettings() {}

	/** Override to provide a default new asset name prefix from SoundCueTemplateFactory */
	virtual FString GenerateDefaultNewAssetName(const TArray<TWeakObjectPtr<USoundWave>>& Waves) const { return FString(); }

	UFUNCTION(BlueprintCallable, Category = Config)
	SOUNDCUETEMPLATES_API void AddSoundWavesToTemplate(const TArray<UObject*>& SelectedObjects);

protected:
	/**
	 * Function to override that uses internal editor-data only properties to re-build node
	 * graph of provided SoundCue.  Provided SoundCue may be a reference to this SoundCue
	 * or one to be copied from this template's graph-building schema.
	 */
	virtual void OnRebuildGraph(USoundCue& SoundCue) const PURE_VIRTUAL(, );

	/**
	 * Node width offset between nodes in the SoundCueEditor.
	 */
	static SOUNDCUETEMPLATES_API float GetNodeWidthOffset();

	/**
	 * Initial width offset of first node in the SoundCueEditor.
	 */
	static SOUNDCUETEMPLATES_API float GetInitialWidthOffset();

	/**
	 * Node height offset between nodes in the SoundCueEditor.
	 */
	static SOUNDCUETEMPLATES_API float GetNodeHeightOffset();

	/**
	 * Initial height offset of first node in the SoundCueEditor.
	 */
	static SOUNDCUETEMPLATES_API float GetInitialHeightOffset();

	/**
	 * Sets the provided node's position to the location on the template's grid
	 */
	static void SetSoundNodePosition(USoundNode& InNode, int32 Column, int32 Row)
	{
		InNode.GraphNode->NodePosX -= Column * GetNodeWidthOffset() + GetInitialWidthOffset();
		InNode.GraphNode->NodePosY += Row * GetNodeHeightOffset() + GetInitialHeightOffset();
	}

	/**
	 * Utility function that returns newly constructed sound node that has been added to provided SoundCue's Node Graph.
	 * SoundCue				- SoundCue to insert the constructed node into.
	 * InParentNode			- If not null, Node to set as parent of constructed child.
	 * Column				- (Optional) If positive, sets visual column of node in built graph as displayed in the SoundCueEditor.  Columns are indexed as positive
	 *						  to the left from the root (index 0).  Can be viewed when using the 'Copy To SoundCue' action of the given template.
	 * Row					- (Optional) If positive, sets visual row of node in built graph as displayed in the SoundCueEditor.  Columns are indexed as positive
	 *						  descending from the root (index 0). Can be viewed when using the 'Copy To SoundCue' action of the given template.
	 * InputPinIndex		- (Optional) If InParentNode set, supplies index of input pin to connect newly constructed ChildNode to.
	 * InputPinDisplayName	- (Optional) Name to set the targeted input pin to.
	 */
	template <class T>
	static T& ConstructSoundNodeChild(USoundCue& SoundCue, USoundNode* InParentNode, int32 Column, int32 Row, int32 InputPinIndex = -1, const FText* InputPinDisplayName = nullptr)
	{
		T* ChildNode = SoundCue.ConstructSoundNode<T>();
		check(ChildNode);

		SetSoundNodePosition(*ChildNode, Column, Row);

		if (InParentNode)
		{
			AddSoundNodeChild(*InParentNode, *ChildNode, InputPinIndex, InputPinDisplayName);
		}

		return *ChildNode;
	}

	/**
	 * Constructs initial root sound node of template. Function will fail if SoundCue already contains a root/first node.
	 * SoundCue	- Parent of initial node to construct.
	 */
	template <class T>
	static T& ConstructSoundNodeRoot(USoundCue& SoundCue)
	{
		T* RootNode = SoundCue.ConstructSoundNode<T>();
		check(RootNode);
		check(!SoundCue.FirstNode);

		SoundCue.FirstNode = RootNode;
		RootNode->GraphNode->NodePosX = -1 * GetInitialWidthOffset();
		RootNode->GraphNode->NodePosY = GetInitialHeightOffset();

		return *RootNode;
	}

	/**
	 * Utility function that adds a provided child node to a parent node's list of children.
	 * InParentNode			- Node to set as parent of provided child.
	 * InChildNode			- Node to set as child of provided parent.
	 * InputPinIndex		- Index of input pin to connect newly constructed ChildNode to.
	 * InputPinDisplayName	- (Optional) Name to set the targeted input pin to.
	 */
	static void AddSoundNodeChild(USoundNode& InParentNode, USoundNode& InChildNode, int32 InputPinIndex, const FText* InputPinDisplayName = nullptr)
	{
		check(InputPinIndex >= 0 && InputPinIndex <= InParentNode.GetMaxChildNodes());

		const int32 NewChildNodeIndex = InParentNode.ChildNodes.Num();
		InParentNode.InsertChildNode(NewChildNodeIndex);
		InParentNode.ChildNodes[NewChildNodeIndex] = &InChildNode;

		UEdGraphNode* OutNodeGraph = InParentNode.GetGraphNode();
		check(OutNodeGraph);
		UEdGraphPin* OutputPin = nullptr;
		for (UEdGraphPin* ActivePin : InChildNode.GetGraphNode()->Pins)
		{
			if (ActivePin->Direction == EGPD_Output)
			{
				OutputPin = ActivePin;
				break;
			}
		}
		check(OutputPin);

		UEdGraphPin* InputPin = nullptr;
		int32 Inputs = 0;
		for (UEdGraphPin* ActivePin : OutNodeGraph->Pins)
		{
			if (ActivePin && ActivePin->Direction == EGPD_Input)
			{
				if (Inputs == InputPinIndex)
				{
					InputPin = ActivePin;
					break;
				}
				Inputs++;
			}
		}

		if (InputPin)
		{
			if (InputPinDisplayName)
			{
				InputPin->PinFriendlyName = *InputPinDisplayName;
			}
			InputPin->MakeLinkTo(OutputPin);
		}
	}
#endif // WITH_EDITOR
};
