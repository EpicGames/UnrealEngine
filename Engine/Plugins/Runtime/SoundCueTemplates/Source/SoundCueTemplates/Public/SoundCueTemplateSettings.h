// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DeveloperSettings.h"

#include "SoundCueTemplateSettings.generated.h"

#define UE_API SOUNDCUETEMPLATES_API


struct FSoundCueTemplateQualitySettingsNotifier
{
	void PostQualitySettingsUpdated() const;
	void OnAudioSettingsChanged() const;
};

USTRUCT()
struct FSoundCueTemplateQualitySettings
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Quality")
	FText DisplayName;

	// The max number of variations to include for the given quality in a SoundCueContainer set to 'Concatenate'.
	UPROPERTY(EditAnywhere, Category = "Quality", meta = (ClampMin = "1"))
	int32 MaxConcatenatedVariations;

	// The max number of variations to include for the given quality in a SoundCueContainer set to 'Randomized'.
	UPROPERTY(EditAnywhere, Category = "Quality", meta = (ClampMin = "1"))
	int32 MaxRandomizedVariations;

	// The max number of variations to include for the given quality in a SoundCueContainer set to 'Mix'.
	UPROPERTY(EditAnywhere, Category = "Quality", meta = (ClampMin = "1"))
	int32 MaxMixVariations;

	FSoundCueTemplateQualitySettings();
};

UCLASS(MinimalAPI, config=Game, defaultconfig, meta=(DisplayName="Sound Cue Templates"))
class USoundCueTemplateSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(config, EditAnywhere, Category = "Quality")
	TArray<FSoundCueTemplateQualitySettings> QualityLevels;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty( struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	UE_API virtual void PostInitProperties() override;

	FSoundCueTemplateQualitySettingsNotifier SettingsNotifier;
public:
	UE_API bool RebuildQualityLevels();

	// Get the quality level settings at the provided level index
	UE_API const FSoundCueTemplateQualitySettings& GetQualityLevelSettings(int32 Index) const;
	UE_API int32 GetQualityLevelSettingsNum() const;
#endif // WITH_EDITOR
};

#undef UE_API
