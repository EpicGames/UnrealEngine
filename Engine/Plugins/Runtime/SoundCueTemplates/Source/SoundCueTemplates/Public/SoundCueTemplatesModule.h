// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

SOUNDCUETEMPLATES_API DECLARE_LOG_CATEGORY_EXTERN(SoundCueTemplates, Log, All);

class FSoundCueTemplatesModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
