// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SoundCueTemplates : ModuleRules
{
    public SoundCueTemplates(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core"
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "CoreUObject",
                "Engine",
                "Slate",
                "SlateCore",
				"DeveloperSettings"
			}
        );
    }
}
