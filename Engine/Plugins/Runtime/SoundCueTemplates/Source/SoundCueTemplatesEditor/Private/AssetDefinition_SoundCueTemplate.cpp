// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_SoundCueTemplate.h"

#include "AssetDefinitionRegistry.h"
#include "AssetToolsModule.h"
#include "AssetTypeActions/AssetDefinition_SoundBase.h"
#include "AudioEditorSettings.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserMenuContexts.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "Misc/PackageName.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Sound/SoundWaveProcedural.h"
#include "SoundCueTemplateFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_SoundCueTemplate)

#define LOCTEXT_NAMESPACE "AssetTypeActions"


TConstArrayView<FAssetCategoryPath> UAssetDefinition_SoundCueTemplate::GetAssetCategories() const
{
	static const auto Pinned_Categories = { EAssetCategoryPaths::Audio };
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Audio, LOCTEXT("AssetSoundCue_SubMenu", "Legacy")) };
	
	if (GetDefault<UAudioEditorSettings>()->bPinSoundCueTemplateInAssetMenu)
	{
		return Pinned_Categories;
	}
	
	return Categories;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_SoundCueTemplate
{
	void ExecuteCopyToSoundCue(const FToolMenuContext& MenuContext)
	{
		const FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
		{
			for (USoundCueTemplate* Object : Context->LoadSelectedObjects<USoundCueTemplate>())
			{
				FString Name;
				FString PackagePath;
				AssetToolsModule.Get().CreateUniqueAssetName(Object->GetOutermost()->GetName(), TEXT(""), PackagePath, Name);

				if (USoundCueTemplateCopyFactory* Factory = NewObject<USoundCueTemplateCopyFactory>())
				{
					Factory->SoundCueTemplate = TWeakObjectPtr<USoundCueTemplate>(Object);
					FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
					ContentBrowserModule.Get().CreateNewAsset(Name, FPackageName::GetLongPackagePath(PackagePath), USoundCue::StaticClass(), Factory);
				}
			}
		}
	}

	void ExecuteCreateSoundCueTemplate(const struct FToolMenuContext& MenuContext)
	{
		const FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
		{
			TArray<USoundWave*> SoundWaves = Context->LoadSelectedObjectsIf<USoundWave>([](const FAssetData& AssetData) { return !AssetData.IsInstanceOf<USoundWaveProcedural>(); });
			if (!SoundWaves.IsEmpty())
			{
				FString PackagePath;
				FString Name;
				
				AssetToolsModule.Get().CreateUniqueAssetName(SoundWaves[0]->GetOutermost()->GetName(), TEXT(""), PackagePath, Name);

				USoundCueTemplateFactory* Factory = NewObject<USoundCueTemplateFactory>();
				Factory->SoundWaves = TArray<TWeakObjectPtr<USoundWave>>(SoundWaves);
				Factory->ConfigureProperties();
				Name = Factory->GetDefaultNewAssetName();

				const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().CreateNewAsset(Name, FPackageName::GetLongPackagePath(PackagePath), USoundCueTemplate::StaticClass(), Factory);
			}
		}
	}

	bool CanCreateSoundCueTemplate(const FToolMenuContext& MenuContext)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
		{
			const TArray<TSoftObjectPtr<USoundWaveProcedural>> SoundWaveProcedurals = Context->GetSelectedAssetSoftObjects<USoundWaveProcedural>();

			if (SoundWaveProcedurals.IsEmpty())
			{
				return true;
			}
		}

		return false;
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

			{
				const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(USoundWave::StaticClass());
				FToolMenuSection* Section = CastChecked<UAssetDefinition_SoundAssetBase>(AssetDefinition)->FindSoundContextMenuSection("Sound");
				check(Section);
				Section->AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					const TAttribute<FText> Label = LOCTEXT("SoundWave_CreateSoundCueTemplate", "Create SoundCueTemplate");
					const TAttribute<FText> ToolTip = LOCTEXT("SoundWave_CreateSoundCueTemplateToolTip", "Creates a SoundCueTemplate from the selected sound waves.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.SoundCue");
					
					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateSoundCueTemplate);
					UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateStatic(&CanCreateSoundCueTemplate);
					InSection.AddMenuEntry("SoundWave_CreateSoundCueTemplate", Label, ToolTip, Icon, UIAction);
				}));
			}


			{
				const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(USoundCueTemplate::StaticClass());
				FToolMenuSection* Section = CastChecked<UAssetDefinition_SoundAssetBase>(AssetDefinition)->FindSoundContextMenuSection("Sound");
				Section->AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
					{
						{
							const TAttribute<FText> Label = LOCTEXT("SoundCueTemplate_CopyToSoundCue", "Copy To Sound Cue");
							const TAttribute<FText> ToolTip = LOCTEXT("SoundCueTemplate_CopyToSoundCueTooltip", "Exports a Sound Cue Template to a Sound Cue.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.SoundCue");
							
							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCopyToSoundCue);
							InSection.AddMenuEntry("SoundCueTemplate_CopyToSoundCue", Label, ToolTip, Icon, UIAction);
						}
					}
				}));
			}
		}));
	});
}
#undef LOCTEXT_NAMESPACE
