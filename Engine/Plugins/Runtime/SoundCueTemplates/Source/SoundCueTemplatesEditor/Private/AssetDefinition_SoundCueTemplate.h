// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SoundCueTemplate.h"
#include "AssetTypeActions/AssetDefinition_SoundBase.h"

#include "AssetDefinition_SoundCueTemplate.generated.h"

struct FAssetCategoryPath;

class USoundCueTemplate;
struct FToolMenuContext;

UCLASS()
class UAssetDefinition_SoundCueTemplate : public UAssetDefinition_SoundBase
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SoundCueTemplate", "Sound Cue Template"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 0, 0)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundCueTemplate::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	// UAssetDefinition End
};
