// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"
#include "ClassViewerFilter.h"

class FSoundCueTemplateAssetParentFilter : public IClassViewerFilter
{
public:
	/** All children of these classes will be included unless filtered out by another setting. */
	TSet< const UClass* > AllowedChildrenOfClasses;

	/** Disallowed class flags. */
	EClassFlags DisallowedClassFlags;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InClass->HasAnyClassFlags(DisallowedClassFlags) && InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags) && InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}
};
