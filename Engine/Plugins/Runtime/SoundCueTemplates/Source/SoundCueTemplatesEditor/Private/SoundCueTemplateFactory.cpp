// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundCueTemplateFactory.h"
#include "SoundCueTemplate.h"
#include "SoundCueTemplateClassFilter.h"

#include "AudioAnalytics.h"
#include "Kismet2/SClassPickerDialog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundCueTemplateFactory)

#define LOCTEXT_NAMESPACE "SoundCueTemplatesEditor"

USoundCueTemplateCopyFactory::USoundCueTemplateCopyFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundCue::StaticClass();

	bCreateNew = false;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundCueTemplateCopyFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (SoundCueTemplate.IsValid())
	{
		if (USoundCue* NewCue = NewObject<USoundCue>(InParent, Name, Flags))
		{
			SoundCueTemplate.Get()->RebuildGraph(*NewCue);
			return NewCue;
		}
	}

	return nullptr;
}

USoundCueTemplateFactory::USoundCueTemplateFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundCueTemplate::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
	SoundCueTemplateClass = nullptr;
}

bool USoundCueTemplateFactory::ConfigureProperties()
{
	SoundCueTemplateClass = nullptr;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	TSharedPtr<FSoundCueTemplateAssetParentFilter> Filter = MakeShared<FSoundCueTemplateAssetParentFilter>();
	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_HideDropDown;
	Filter->AllowedChildrenOfClasses.Add(USoundCueTemplate::StaticClass());

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.ClassFilters.Add(Filter.ToSharedRef());

	const FText TitleText = LOCTEXT("CreateSoundCueTemplateOfType", "Pick Type of SoundCueTemplate");
	UClass* ChosenClass = nullptr;

	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, USoundCueTemplate::StaticClass());

	if (bPressedOk)
	{
		SoundCueTemplateClass = ChosenClass;
	}
	return bPressedOk;
}

UObject* USoundCueTemplateFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (SoundCueTemplateClass != nullptr)
	{
		if (USoundCueTemplate* NewSoundCueTemplate = NewObject<USoundCueTemplate>(InParent, SoundCueTemplateClass, Name, Flags))
		{
			NewSoundCueTemplate->LoadTemplateDefaultSettings();

			if (SoundWaves.Num())
			{
				NewSoundCueTemplate->AddSoundWaves(SoundWaves);
			}

			NewSoundCueTemplate->RebuildGraph(*NewSoundCueTemplate);

			Audio::Analytics::RecordEvent_Usage(TEXT("SoundCueTemplates.TemplateCreated"));

			return NewSoundCueTemplate;
		}
	}

	return nullptr;
}

FString USoundCueTemplateFactory::GetDefaultNewAssetName() const
{
	if (SoundCueTemplateClass)
	{
		FString TemplatePrefix;
		if (const USoundCueTemplate* TemplateCDO = SoundCueTemplateClass.GetDefaultObject())
		{
			TemplatePrefix = TemplateCDO->GenerateDefaultNewAssetName(SoundWaves);
		}

		if (TemplatePrefix.IsEmpty())
		{
			SoundCueTemplateClass->GetName(TemplatePrefix);
		}

		return TemplatePrefix + FString(TEXT("_"));
	}

	return Super::GetDefaultNewAssetName();
}

USoundCueTemplateClassTemplate::USoundCueTemplateClassTemplate(const FObjectInitializer& ObjectInitializer)
	: UPluginClassTemplate(ObjectInitializer)
{
	PluginName = TEXT("SoundCueTemplates");
	SetGeneratedBaseClass(USoundCueTemplate::StaticClass());
}
#undef LOCTEXT_NAMESPACE // "SoundCueTemplatesEditor"

