// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundCueTemplatesEditorModule.h"

#include "ToolMenus.h"

void FSoundCueTemplatesEditorModule::StartupModule()
{
	LLM_SCOPE(ELLMTag::AudioMisc);
}

void FSoundCueTemplatesEditorModule::ShutdownModule()
{
	LLM_SCOPE(ELLMTag::AudioMisc);
	UToolMenus::UnregisterOwner(UE_MODULE_NAME);
}

IMPLEMENT_MODULE(FSoundCueTemplatesEditorModule, SoundCueTemplatesEditor);
