// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClassTemplateEditorSubsystem.h"
#include "Factories/Factory.h"

#include "SoundCueTemplateFactory.generated.h"

class USoundCueTemplate;
class USoundWave;

UCLASS(hidecategories = Object, MinimalAPI)
class USoundCueTemplateCopyFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ End UFactory Interface

public:
	UPROPERTY()
	TWeakObjectPtr<USoundCueTemplate> SoundCueTemplate;
};

UCLASS(hidecategories = Object, MinimalAPI)
class USoundCueTemplateFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = SoundCueTemplateFacotry)
	TSubclassOf<USoundCueTemplate> SoundCueTemplateClass;

	UPROPERTY(EditAnywhere, Category = SoundCueTemplateFactory)
	TArray<TWeakObjectPtr<USoundWave>> SoundWaves;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FString GetDefaultNewAssetName() const override;
	//~ End UFactory Interface
};

UCLASS()
class USoundCueTemplateClassTemplate : public UPluginClassTemplate
{
	GENERATED_UCLASS_BODY()
};
