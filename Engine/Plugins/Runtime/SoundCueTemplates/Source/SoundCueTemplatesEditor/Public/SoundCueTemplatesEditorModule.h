// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

SOUNDCUETEMPLATESEDITOR_API DECLARE_LOG_CATEGORY_EXTERN(SoundCueTemplatesEditor, Log, All);

class FSoundCueTemplatesEditorModule :	public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
