// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SoundCueTemplatesEditor : ModuleRules
{
	public SoundCueTemplatesEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"SoundCueTemplates",
				"AudioEditor",
				"AssetDefinition",
				"ToolMenus"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				
				"Engine",
				"GameProjectGeneration",
				"UnrealEd",
				"Slate",
				"SlateCore",
				"ContentBrowser"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetTools"
			}
		);
	}
}
